// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITDSpatializer.h"
#include "DSP/Delay.h"
#include "HAL/IConsoleManager.h"
#include "ITDSpatializationSourceSettings.h"

#define DEBUG_BYPASS_ITD 0

const int32 NumOutputChannels = 2;

static float SpeedOfSoundCVar = 343.0f;
FAutoConsoleVariableRef CVarSpeedOfSound(
	TEXT("au.itd.SetSpeedOfSound"),
	SpeedOfSoundCVar,
	TEXT("Sets speed of sound to use for ITD calculations.\n")
	TEXT("Value: Speed of sound in meters."),
	ECVF_Default);

static float HeadWidthCVar = 34.0f;
FAutoConsoleVariableRef CVarHeadWidth(
	TEXT("au.itd.SetHeadWidth"),
	HeadWidthCVar,
	TEXT("Sets the listener's head width from ear to ear, in centimeters.\n")
	TEXT("Value: The listener's head width from ear to ear, in centimeters."),
	ECVF_Default);

static float InterpolationTauCVar = 0.1f;
FAutoConsoleVariableRef CVarInterpolationTime(
	TEXT("au.itd.SetInterpolationTime"),
	InterpolationTauCVar,
	TEXT("Sets how quickly the audio renderer follows the objects position, in seconds.\n")
	TEXT("Value: Interpolation time, in seconds."),
	ECVF_Default);

static int32 EnableILDCVar = 1;
FAutoConsoleVariableRef CVarEnablePanning(
	TEXT("au.itd.EnableILD"),
	EnableILDCVar,
	TEXT("Sets whether we should use level differences in addition to delay.\n")
	TEXT("0: ILD disabled, 1: ILD enabled."),
	ECVF_Default);

FSourceSpatializer::FSourceSpatializer()
{
	// Privatized default constructor. Should not be used.
}

FSourceSpatializer::FSourceSpatializer(float InSampleRate)
{
	// Max delay line length will be the maximum distance audio will need to travel divided by the speed of sound:
	const float MaxDelay = 0.5f; //HeadWidthCVar / 1000.0f / SpeedOfSoundCVar;
	const float EaseFactor = Audio::FExponentialEase::GetFactorForTau(InterpolationTauCVar, InSampleRate);

	LeftDelays.Reset();
	LeftDelays.AddDefaulted(2);

	RightDelays.Reset();
	RightDelays.AddDefaulted(2);

	for (Audio::FDelay& Delay : LeftDelays)
	{
		Delay.Init(InSampleRate, MaxDelay);
		Delay.SetEaseFactor(EaseFactor);
	}

	for (Audio::FDelay& Delay : RightDelays)
	{
		Delay.Init(InSampleRate, MaxDelay);
		Delay.SetEaseFactor(EaseFactor);
	}
	
	LeftGain.SetEaseFactor(EaseFactor);
	RightGain.SetEaseFactor(EaseFactor);
}

FSourceSpatializer::~FSourceSpatializer()
{
	// Wait for ProcessSource.
	FScopeLock ScopeLock(&DestructorCriticalSection);
}

void FSourceSpatializer::ProcessSource(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData)
{
	if (!DestructorCriticalSection.TryLock())
	{
		return;
	}

#if DEBUG_BYPASS_ITD
	FMemory::Memcpy(OutputData.AudioBuffer.GetData(), InputData.AudioBuffer->GetData(), OutputData.AudioBuffer.Num() * sizeof(float));
#else
	EvaluateGainDestinations(InputData);
	EvaluateDelayDestinations(InputData);

	const float* InAudio = InputData.AudioBuffer->GetData();
	float* OutAudio = OutputData.AudioBuffer.GetData();

	const int32 NumInputChannels = InputData.NumChannels;

	int32 OutFrameIndex = 0;

	for (int32 InFrameIndex = 0; InFrameIndex < InputData.AudioBuffer->Num(); InFrameIndex += NumInputChannels)
	{
		for (int32 InChannelIndex = 0; InChannelIndex < NumInputChannels; InChannelIndex++)
		{
			OutAudio[OutFrameIndex] += LeftDelays[InChannelIndex].ProcessAudioSample(InAudio[InFrameIndex + InChannelIndex]) * LeftGain.GetNextValue();
			OutAudio[OutFrameIndex + 1] += RightDelays[InChannelIndex].ProcessAudioSample(InAudio[InFrameIndex + InChannelIndex]) * RightGain.GetNextValue();
		}

		OutFrameIndex += NumOutputChannels;
	}
#endif

	DestructorCriticalSection.Unlock();
}

void FSourceSpatializer::ZeroOut()
{
	FScopeLock ScopeLock(&DestructorCriticalSection);

	for (Audio::FDelay& Delay : LeftDelays)
	{
		Delay.Reset();
	}

	for (Audio::FDelay& Delay : RightDelays)
	{
		Delay.Reset();
	}

	LeftGain.SetValue(1.0f, true);
	RightGain.SetValue(1.0f, true);
}

void FSourceSpatializer::SetILDCurve(const FRuntimeFloatCurve& InCurve)
{
	CurrentILDCurve = InCurve;
}

void FSourceSpatializer::EvaluateGainDestinations(const FAudioPluginSourceInputData& InputData)
{
	const FRichCurve* InRichCurve = CurrentILDCurve.GetRichCurveConst();

	if (!EnableILDCVar || !InRichCurve)
	{
		LeftGain.SetValue(1.0f);
		RightGain.SetValue(1.0f);
		return;
	}

	const float DistanceFactor = FMath::Clamp<float>(InRichCurve->Eval(InputData.SpatializationParams->Distance, 0.0f), 0.0f, 1.0f);

	const float HeadRadius = (HeadWidthCVar / 100.0f) * 0.5f;

	// PanValue value normalized [0, 1]:
	const float NormalizedPanValue = (FMath::Clamp<float>(InputData.SpatializationParams->EmitterPosition.Y, -1.0f * HeadRadius, HeadRadius) / HeadRadius + 1.0f) * 0.5f;
	const float GainDelta = 0.5f * NormalizedPanValue * DistanceFactor;

	// LeftGain.SetValue(2.0f - GainDelta - DistanceFactor);
	// RightGain.SetValue(GainDelta + 1.0f - DistanceFactor);

	LeftGain.SetValue(0.5f - GainDelta);
	RightGain.SetValue(0.5f + GainDelta);
}

void FSourceSpatializer::EvaluateDelayDestinations(const FAudioPluginSourceInputData& InputData)
{
	if (InputData.NumChannels == 1)
	{
		EvaluateDelayDestinationForInputChannel(0, InputData.SpatializationParams->EmitterPosition.X, InputData.SpatializationParams->EmitterPosition.Y);
	}
	else if (InputData.NumChannels == 2)
	{
		EvaluateDelayDestinationForInputChannel(0, InputData.SpatializationParams->LeftChannelPosition.X, InputData.SpatializationParams->LeftChannelPosition.Y);
		EvaluateDelayDestinationForInputChannel(1, InputData.SpatializationParams->RightChannelPosition.X, InputData.SpatializationParams->RightChannelPosition.Y);
	}
}

void FSourceSpatializer::EvaluateDelayDestinationForInputChannel(int32 ChannelIndex, float X, float Y)
{
	const float HeadRadius = (HeadWidthCVar / 100.0f) * 0.5f;

	const float DistanceToLeftEar = FMath::Sqrt((X * X) + FMath::Square(HeadRadius + Y));
	const float DistanceToRightEar = FMath::Sqrt((X * X) + FMath::Square(HeadRadius - Y));

	const float DeltaInSeconds = (DistanceToLeftEar - DistanceToRightEar) / SpeedOfSoundCVar;

	if (DeltaInSeconds > 0)
	{
		LeftDelays[ChannelIndex].SetEasedDelayMsec(DeltaInSeconds * 1000.0f);
		RightDelays[ChannelIndex].SetEasedDelayMsec(0.0f);
	}
	else
	{
		LeftDelays[ChannelIndex].SetEasedDelayMsec(0.0f);
		RightDelays[ChannelIndex].SetEasedDelayMsec(DeltaInSeconds * -1000.0f);
	}
}

FITDSpatialization::FITDSpatialization()
	: SampleRate(0.0f)
{

}

FITDSpatialization::~FITDSpatialization()
{
}

void FITDSpatialization::Initialize(const FAudioPluginInitializationParams InitializationParams)
{
	SampleRate = InitializationParams.SampleRate;

	Sources.Reset();
	for (uint32 Index = 0; Index < InitializationParams.NumSources; Index++)
	{
		Sources.Emplace(SampleRate);
	}
}

void FITDSpatialization::Shutdown()
{
	Sources.Reset();
}

void FITDSpatialization::OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, USpatializationPluginSourceSettingsBase* InSettings)
{
	Sources[SourceId].ZeroOut();

	UITDSpatializationSourceSettings* SourceSettings = Cast<UITDSpatializationSourceSettings>(InSettings);

	if (SourceSettings && SourceSettings->bEnableILD)
	{
		Sources[SourceId].SetILDCurve(SourceSettings->PanningIntensityOverDistance);
	}
}

void FITDSpatialization::OnReleaseSource(const uint32 SourceId)
{
}

void FITDSpatialization::ProcessAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData)
{
	Sources[InputData.SourceId].ProcessSource(InputData, OutputData);
}

UClass* FITDSpatializationPluginFactory::GetCustomSpatializationSettingsClass() const
{
	return UITDSpatializationSourceSettings::StaticClass();
}
