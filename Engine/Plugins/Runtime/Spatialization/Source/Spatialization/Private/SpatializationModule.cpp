// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatializationModule.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"


void FSpatializationModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(FITDSpatializationPluginFactory::GetModularFeatureName(), &PluginFactory);
}

void FSpatializationModule::ShutdownModule()
{
	// Nothing done here for now.
}

IMPLEMENT_MODULE(FSpatializationModule, Spatialization)
