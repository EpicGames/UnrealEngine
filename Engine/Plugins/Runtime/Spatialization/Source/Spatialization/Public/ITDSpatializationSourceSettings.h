// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioExtensionPlugin.h"
#include "CoreMinimal.h"
#include "Curves/CurveFloat.h"
#include "ITDSpatializationSourceSettings.generated.h"

UCLASS(MinimalAPI)
class UITDSpatializationSourceSettings : public USpatializationPluginSourceSettingsBase
{
	GENERATED_BODY()

public:

	/* Whether we should use any level difference between the left and right channel in our spatialization algorithm. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = SpatializationSettings, meta = (DisplayName = "Enable Level Panning"))
	bool bEnableILD;

	/* This curve will map the intensity of panning (y-axis, 0.0-1.0) as a factor of distance (in Unreal Units) */
	UPROPERTY(GlobalConfig, EditAnywhere, BlueprintReadWrite, Category = SpatializationSettings)
	FRuntimeFloatCurve PanningIntensityOverDistance;
};

