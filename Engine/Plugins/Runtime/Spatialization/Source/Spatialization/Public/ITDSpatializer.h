// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curves/CurveFloat.h"
#include "IAudioExtensionPlugin.h"
#include "DSP/Delay.h"

struct FSourceSpatializer
{
	FSourceSpatializer(float InSampleRate);
	~FSourceSpatializer();

	void ProcessSource(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData);

	// Zeros out all delay lines.
	void ZeroOut();

	void SetILDCurve(const FRuntimeFloatCurve& InCurve);

private:
	FSourceSpatializer();

	void EvaluateGainDestinations(const FAudioPluginSourceInputData& InputData);
	void EvaluateDelayDestinations(const FAudioPluginSourceInputData& InputData);
	void EvaluateDelayDestinationForInputChannel(int32 ChannelIndex, float X, float Y);


	// Each input channel requires a separate delay line for the left and right output channels:
	TArray<Audio::FDelay> LeftDelays;
	TArray<Audio::FDelay> RightDelays;

	Audio::FExponentialEase LeftGain;
	Audio::FExponentialEase RightGain;

	FCriticalSection DestructorCriticalSection;

	FRuntimeFloatCurve CurrentILDCurve;
};

class FITDSpatialization : public IAudioSpatialization
{
public:
	FITDSpatialization();
	~FITDSpatialization();

	/** IAudioSpatialization implementation */
	virtual void Initialize(const FAudioPluginInitializationParams InitializationParams) override;
	virtual void Shutdown() override;
	virtual void OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, USpatializationPluginSourceSettingsBase* InSettings) override;
	virtual void OnReleaseSource(const uint32 SourceId) override;
	virtual void ProcessAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData);
	/** End of IAudioSpatialization implementation */

private:
	TArray<FSourceSpatializer> Sources;

	float SampleRate;
};

class FITDSpatializationPluginFactory : public IAudioSpatializationFactory
{
public:
	virtual FString GetDisplayName() override
	{
		static FString DisplayName = FString(TEXT("Simple ITD"));
		return DisplayName;
	}

	virtual bool SupportsPlatform(const FString& PlatformName) override
	{
		return true;
	}

	virtual TAudioSpatializationPtr CreateNewSpatializationPlugin(FAudioDevice* OwningDevice) override
	{
		return TAudioSpatializationPtr(new FITDSpatialization());
	};

	virtual int32 GetMaxSupportedChannels() override
	{
		return 2;
	}

	virtual UClass* GetCustomSpatializationSettingsClass() const override;

};
