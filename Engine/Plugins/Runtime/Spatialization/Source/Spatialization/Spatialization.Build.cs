// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;

public class Spatialization : ModuleRules
{
	public Spatialization(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"SignalProcessing",
				"AudioExtensions"
			}
			);

        PrivateIncludePathModuleNames.Add("TargetPlatform");
    }
}
