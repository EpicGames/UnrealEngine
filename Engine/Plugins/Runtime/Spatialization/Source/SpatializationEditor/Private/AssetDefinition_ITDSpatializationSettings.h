// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "ITDSpatializationSourceSettings.h"
#include "AssetDefinition_ITDSpatializationSettings.generated.h"

UCLASS()
class UAssetDefinition_ITDSpatializationSettings : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_ITDSpatializationSettings", "ITD Source Spatialization Settings"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(145, 145, 145); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UITDSpatializationSourceSettings::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_ITDSpatializationSettingsSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_ITDSpatializationSettingsSubMenuSection", "Binaural Spatialization"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	// UAssetDefinition End
};
