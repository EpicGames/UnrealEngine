// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITDSpatializationSourceSettingsFactory.h"
#include "AudioAnalytics.h"
#include "ITDSpatializationSourceSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ITDSpatializationSourceSettingsFactory)

UITDSpatializationSettingsFactory::UITDSpatializationSettingsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UITDSpatializationSourceSettings::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UITDSpatializationSettingsFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags,
	UObject* Context, FFeedbackContext* Warn)
{
	Audio::Analytics::RecordEvent_Usage("Spatialization.SettingsCreated");
	return NewObject<UITDSpatializationSourceSettings>(InParent, InName, Flags);
}

uint32 UITDSpatializationSettingsFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Sounds;
}
