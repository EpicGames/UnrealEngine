// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioExtensionPlugin.h"
#include "AssetToolsModule.h"
#include "Factories/Factory.h"
#include "ITDSpatializationSourceSettingsFactory.generated.h"

UCLASS(MinimalAPI, hidecategories = Object)
class UITDSpatializationSettingsFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

		virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context,
			FFeedbackContext* Warn) override;

	virtual uint32 GetMenuCategories() const override;
};

