// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;

public class SpatializationEditor : ModuleRules
{
	public SpatializationEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "AudioEditor"
            }
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"CoreUObject",
				"EditorFramework",
				"Engine",
				"AudioMixer",
                "LevelEditor",
				"Spatialization",
				"UnrealEd"
            }
			);

        PrivateIncludePathModuleNames.Add("TargetPlatform");
    }
}
