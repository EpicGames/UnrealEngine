{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Spatialization",
	"Description": "Plugin featuring a variety of basic audio spatialization solutions.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "Spatialization",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "SpatializationEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	]
}